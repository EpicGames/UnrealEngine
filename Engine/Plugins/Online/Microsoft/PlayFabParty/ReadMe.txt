; You will need the following config added to one of your title's GDK Engine Ini files.  Set the PlayFab title ID with your own from the PlayFab website.

[/Script/PlayFabParty.PlayFabPartyNetDriver]
NetConnectionClassName="/Script/PlayFabParty.PlayFabPartyNetConnection"

[/Script/Engine.Engine]
!NetDriverDefinitions=ClearArray
+NetDriverDefinitions=(DefName="GameNetDriver",DriverClassName="/Script/PlayFabParty.PlayFabPartyNetDriver",DriverClassNameFallback="/Script/PlayFabParty.PlayFabPartyNetDriver")
+NetDriverDefinitions=(DefName="BeaconNetDriver",DriverClassName="/Script/PlayFabParty.PlayFabPartyNetDriver",DriverClassNameFallback="/Script/PlayFabParty.PlayFabPartyNetDriver")
+NetDriverDefinitions=(DefName="DemoNetDriver",DriverClassName="/Script/Engine.DemoNetDriver",DriverClassNameFallback="/Script/Engine.DemoNetDriver")

[PlayFab]
; Your Hex PlayFab Title ID String Here
AppId=
; Time in seconds to wait after a login failure before trying again
LoginFailureDelaySeconds=15
; Time in seconds to wait after a successful login to refresh xbox login credentials. The recommended amount is half of the length of validity of the token
LoginRefreshDelaySeconds=7200
