This plugin is experimental and meant to be something to jump start your development. This plugin has been tested using ShooterGame on Windows

It does not meet the Epic Games Store requirement for cross store support as is. You will need to add support to ship on EGS.

This plugin depends on the Engine/Source/ThirdParty/EOSSDK module which houses the SDK. Please refer to that module for information on installing / upgrading the SDK

This plugin supports:
	- EAS/EGS Auth
	- EAS Presence
	- EAS Friends
	- EGS/EAS Ecom
	- EOS Sessions
	- EOS Stats
	- EOS Achievements
	- EOS Leaderboards
	- EOS p2p Sockets
	- EOS Metrics (session based analytics)
	- EOS Voice

To configure your settings:
	1. Go to Project Settings
	2. Scroll down to Plugins
	3. Select Epic Online Services
	4. Add a entry in the list for each artifact on EGS or use the default name for non-EGS
		- Each deployment id/artifact can have separate settings
